{
	"FileVersion": 3,

	"FriendlyName": "Online Base",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Shared code online subsystem (OSSv1) and online service (OSSv2) interfaces.",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": true,

	"Modules": [
		{
			"Name": "OnlineBase",
			"Type": "Runtime"
		}
	]
}