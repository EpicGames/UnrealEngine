// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class OnlineBase : ModuleRules
	{
		public OnlineBase(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Sockets"
				}
			);

			// NOTE:  OnlineBase cannot depend on Engine!
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject"
				}
			);
		}
	}
}