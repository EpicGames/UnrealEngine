// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/LANBeacon.h"
#include "Misc/FeedbackContext.h"
#include "UObject/CoreNet.h"
#include "SocketSubsystem.h"
#include "Sockets.h"
#include "Online/NboSerializer.h"

/** Sets the broadcast address for this object */
FLanBeacon::FLanBeacon(void) 
	: ListenSocket(NULL),
	  SockAddr(ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr())
{
}

/** Frees the broadcast socket */
FLanBeacon::~FLanBeacon(void)
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	SocketSubsystem->DestroySocket(ListenSocket);
}

/** Return true if there is a valid ListenSocket */
bool FLanBeacon::IsListenSocketValid() const
{
	return (ListenSocket ? true : false);
}

/**
 * Initializes the socket
 *
 * @param Port the port to listen on
 *
 * @return true if both socket was created successfully, false otherwise
 */
bool FLanBeacon::Init(int32 Port)
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	bool bSuccess = false;
	// Now the listen address
	ListenAddr = SocketSubsystem->GetLocalBindAddr(*GWarn);
	ListenAddr->SetPort(Port);
	// Set our broadcast address
	BroadcastAddr = SocketSubsystem->CreateInternetAddr(ListenAddr->GetProtocolType());
	BroadcastAddr->SetBroadcastAddress();
	BroadcastAddr->SetPort(Port);
	// A temporary "received from" address
	SockAddr = SocketSubsystem->CreateInternetAddr();

	// Now create and set up our sockets (no VDP)
	ListenSocket = SocketSubsystem->CreateSocket(NAME_DGram, TEXT("LAN beacon"), ListenAddr->GetProtocolType());
	if (ListenSocket != NULL)
	{
		ListenSocket->SetReuseAddr();
		ListenSocket->SetNonBlocking();
		ListenSocket->SetRecvErr();
		// Bind to our listen port
		if (ListenSocket->Bind(*ListenAddr))
		{
			// Set it to broadcast mode, so we can send on it
			// NOTE: You must set this to broadcast mode on Xbox 360 or the
			// secure layer will eat any packets sent
			bSuccess = ListenSocket->SetBroadcast();
		}
		else
		{
			UE_LOGF(LogTemp, Error, "Failed to bind listen socket to addr (%ls) for LAN beacon",
				*ListenAddr->ToString(true));
		}
	}
	else
	{
		UE_LOGF(LogTemp, Error, "Failed to create listen socket for LAN beacon");
	}
	return bSuccess && ListenSocket;
}

/**
 * Called to poll the socket for pending data. Any data received is placed
 * in the specified packet buffer
 *
 * @param PacketData the buffer to get the socket's packet data
 * @param BufferSize the size of the packet buffer
 *
 * @return the number of bytes read (<= 0 if none or an error)
 */
int32 FLanBeacon::ReceivePacket(uint8* PacketData, int32 BufferSize)
{
	check(PacketData && BufferSize);
	// Default to no data being read
	int32 BytesRead = 0;
	if (ListenSocket != NULL)
	{
		// Read from the socket
		ListenSocket->RecvFrom(PacketData, BufferSize, BytesRead, *SockAddr);
		if (BytesRead > 0)
		{
			UE_LOGF(LogTemp, Verbose, "Received %d bytes from %ls", BytesRead, *SockAddr->ToString(true));
		}
	}

	return BytesRead;
}

/**
 * Uses the cached broadcast address to send packet to a subnet
 *
 * @param Packet the packet to send
 * @param Length the size of the packet to send
 */
bool FLanBeacon::BroadcastPacket(uint8* Packet, int32 Length)
{
	int32 BytesSent = 0;
	UE_LOGF(LogTemp, Verbose, "Sending %d bytes to %ls", Length, *BroadcastAddr->ToString(true));
	return ListenSocket->SendTo(Packet, Length, BytesSent, *BroadcastAddr) && (BytesSent == Length);
}

/**
* Creates the LAN beacon for queries/advertising servers
*/
bool FLANSession::Host(FOnValidQueryPacketDelegate& QueryDelegate)
{
	bool bSuccess = false;
	if (LanBeacon != NULL)
	{
		StopLANSession();
	}

	LanBeaconGuid = FGuid::NewGuid();

	// Bind a socket for LAN beacon activity
	LanBeacon = new FLanBeacon();
	if (LanBeacon->Init(LanAnnouncePort))
	{
		OnValidQueryPacketDelegates.Add(QueryDelegate);
		// We successfully created everything so mark the socket as needing polling
		LanBeaconState = ELanBeaconState::Hosting;
		bSuccess = true;
		UE_LOGF(LogTemp, Verbose, "Listening for LAN beacon requests on %d",	LanAnnouncePort);
	}
	else
	{
		UE_LOGF(LogTemp, Warning, "Failed to init to LAN beacon %ls",	ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetSocketError());
	}
	
	return bSuccess;
}

/**
 * Creates the LAN beacon for queries/advertising servers
 *
 * @param Nonce unique identifier for this search
 * @param ResponseDelegate delegate to fire when a server response is received
 */
bool FLANSession::Search(FNboSerializeToBuffer& Packet, FOnValidResponsePacketDelegate& ResponseDelegate, FOnSearchingTimeoutDelegate& TimeoutDelegate)
{
	bool bSuccess = true;
	if (LanBeacon != NULL)
	{
		StopLANSession();
	}
	
	LanBeaconGuid = FGuid::NewGuid();

	// Bind a socket for LAN beacon activity
	LanBeacon = new FLanBeacon();
	if (LanBeacon->Init(LanAnnouncePort) == false)
	{
		UE_LOGF(LogTemp, Warning, "Failed to create socket for lan announce port %ls", ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetSocketError());
		bSuccess = false;
	}

	// If we have a socket and a nonce, broadcast a discovery packet
	if (LanBeacon && bSuccess)
	{
		// Now kick off our broadcast which hosts will respond to
		if (LanBeacon->BroadcastPacket(Packet, Packet.GetByteCount()))
		{
			UE_LOGF(LogTemp, Verbose, "Sent query packet...");
			// We need to poll for the return packets
			LanBeaconState = ELanBeaconState::Searching;
			// Set the timestamp for timing out a search
			LanQueryTimeLeft = LanQueryTimeout;

			RetryData.Reset();
			RetryData.Append(Packet.GetBuffer().GetData(), Packet.GetByteCount());

			LanQueryRetryTimeLeft = LanQueryRetryTime;

			OnValidResponsePacketDelegates.Add(ResponseDelegate);
			OnSearchTimeoutDelegates.Add(TimeoutDelegate);
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "Failed to send discovery broadcast %ls", ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetSocketError());
			bSuccess = false;
		}
	}

	return bSuccess;
}

/** Stops the LAN beacon from accepting broadcasts */
void FLANSession::StopLANSession()
{
	// Don't poll anymore since we are shutting it down
	LanBeaconState = ELanBeaconState::NotUsingLanBeacon;

	// Unbind the LAN beacon object
	if (LanBeacon)
	{	
		delete LanBeacon;
		LanBeacon = NULL;
	}

	// Clear delegates
	OnValidQueryPacketDelegates.Clear();
	OnValidResponsePacketDelegates.Clear();
	OnSearchTimeoutDelegates.Clear();

	RetryData.Empty();
}

void FLANSession::Tick(float DeltaTime)
{
	if (LanBeaconState == ELanBeaconState::NotUsingLanBeacon)
	{
		return;
	}

	uint8 PacketData[LAN_BEACON_MAX_PACKET_SIZE];
	bool bShouldRead = true;
	// Read each pending packet and pass it out for processing
	while (bShouldRead)
	{
		int32 NumRead = LanBeacon->ReceivePacket(PacketData, LAN_BEACON_MAX_PACKET_SIZE);
		if (NumRead > 0)
		{
			// Check our mode to determine the type of allowed packets
			if (LanBeaconState == ELanBeaconState::Hosting)
			{
				uint64 ClientNonce;
				// We can only accept Server Query packets
				if (IsValidLanQueryPacket(PacketData, NumRead, ClientNonce))
				{
					// Strip off the header
					OnValidQueryPacketDelegates.Broadcast(&PacketData[LAN_BEACON_PACKET_HEADER_SIZE], NumRead - LAN_BEACON_PACKET_HEADER_SIZE, ClientNonce);
				}
			}
			else if (LanBeaconState == ELanBeaconState::Searching)
			{
				FGuid ResponseGuid;

				// We can only accept Server Response packets
				if (IsValidLanResponsePacket(PacketData, NumRead, ResponseGuid))
				{
					// Strip off the header
					OnValidResponsePacketDelegates.Broadcast(&PacketData[LAN_BEACON_PACKET_HEADER_SIZE], NumRead - LAN_BEACON_PACKET_HEADER_SIZE);
				}
			}
		}
		else
		{
			if (LanBeaconState == ELanBeaconState::Searching)
			{
				LanQueryRetryTimeLeft -= DeltaTime;
				
				if (LanBeacon && (LanQueryRetryTimeLeft <= 0.f) && (LanQueryTimeLeft > LanQueryRetryTime))
				{
					if (LanBeacon->BroadcastPacket(RetryData.GetData(), RetryData.Num()))
					{
						UE_LOGF(LogTemp, Verbose, "Sent retry query packet...");
					}
					else
					{
						UE_LOGF(LogTemp, Warning, "Failed to send retry discovery broadcast %ls", ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetSocketError());
					}

					LanQueryRetryTimeLeft = LanQueryRetryTime;
				}

				// Decrement the amount of time remaining
				LanQueryTimeLeft -= DeltaTime;
				// Check for a timeout on the search packet
				if (LanQueryTimeLeft <= 0.f)
				{
					OnSearchTimeoutDelegates.Broadcast();
				}
			}
			bShouldRead = false;
		}
	}
}

void FLANSession::CreateHostResponsePacket(FNboSerializeToBuffer& Packet, uint64 ClientNonce)
{
	// Add the supported version
	Packet << LAN_BEACON_PACKET_VERSION
		// Platform information
		<< (uint8)FPlatformProperties::IsLittleEndian()
		// Game id to prevent cross game lan packets
		<< LanGameUniqueId
		// Add the packet type
		<< LAN_SERVER_RESPONSE1 << LAN_SERVER_RESPONSE2
		// Append the client nonce as a uint64
		<< ClientNonce
		// Append the beacon guid
		<< LanBeaconGuid;
}

void FLANSession::CreateClientQueryPacket(FNboSerializeToBuffer& Packet, uint64 ClientNonce)
{
	// Build the discovery packet
	Packet << LAN_BEACON_PACKET_VERSION
		// Platform information
		<< (uint8)FPlatformProperties::IsLittleEndian()
		// Game id to prevent cross game lan packets
		<< LanGameUniqueId
		// Identify the packet type
		<< LAN_SERVER_QUERY1 << LAN_SERVER_QUERY2
		// Append the nonce as a uint64
		<< ClientNonce
		// Append the beacon guid
		<< LanBeaconGuid;
}

/**
 * Uses the cached broadcast address to send packet to a subnet
 *
 * @param Packet the packet to send
 * @param Length the size of the packet to send
 */
bool FLANSession::BroadcastPacket(uint8* Packet, int32 Length)
{
	bool bSuccess = false;
	if (LanBeacon)
	{
		bSuccess = LanBeacon->BroadcastPacket(Packet, Length);
		if (!bSuccess)
		{
			UE_LOGF(LogTemp, Warning, "Failed to send broadcast packet %d", (int32)ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLastErrorCode());
		}
	}

	return bSuccess;
}

/**
 * Determines if the packet header is valid or not
 *
 * @param Packet the packet data to check
 * @param Length the size of the packet buffer
 * @param ClientNonce the client nonce contained within the packet
 *
 * @return true if the header is valid, false otherwise
 */
bool FLANSession::IsValidLanQueryPacket(const uint8* Packet, uint32 Length, uint64& ClientNonce)
{
	ClientNonce = 0;
	bool bIsValid = false;
	// Serialize out the data if the packet is the right size
	if (Length == LAN_BEACON_PACKET_HEADER_SIZE)
	{
		FNboSerializeFromBuffer PacketReader(Packet,Length);
		uint8 Version = 0;
		PacketReader >> Version;
		// Do the versions match?
		if (Version == LAN_BEACON_PACKET_VERSION)
		{
			uint8 Platform = 255;
			PacketReader >> Platform;
			// Can we communicate with this platform?
			if (Platform & LanPacketPlatformMask)
			{
				int32 GameId = -1;
				PacketReader >> GameId;
				// Is this our game?
				if (GameId == LanGameUniqueId)
				{
					uint8 SQ1 = 0;
					PacketReader >> SQ1;
					uint8 SQ2 = 0;
					PacketReader >> SQ2;
					// Is this a server query?
					bIsValid = (SQ1 == LAN_SERVER_QUERY1 && SQ2 == LAN_SERVER_QUERY2);
					// Read the client nonce as the outvalue
					PacketReader >> ClientNonce;

					// Read the client's beacon guid 
					FGuid BeaconGuid;
					PacketReader >> BeaconGuid;
				}
			}
		}
	}
	return bIsValid;
}

/**
 * Determines if the packet header is valid or not
 *
 * @param Packet the packet data to check
 * @param Length the size of the packet buffer
 * @param ResponseGuid the lan beacon guid in the response packet
 *
 * @return true if the header is valid, false otherwise
 */
bool FLANSession::IsValidLanResponsePacket(const uint8* Packet, uint32 Length, FGuid& ResponseGuid)
{
	bool bIsValid = false;
	// Serialize out the data if the packet is the right size
	if (Length > LAN_BEACON_PACKET_HEADER_SIZE)
	{
		FNboSerializeFromBuffer PacketReader(Packet,Length);
		uint8 Version = 0;
		PacketReader >> Version;
		// Do the versions match?
		if (Version == LAN_BEACON_PACKET_VERSION)
		{
			uint8 Platform = 255;
			PacketReader >> Platform;
			// Can we communicate with this platform?
			if (Platform & LanPacketPlatformMask)
			{
				int32 GameId = -1;
				PacketReader >> GameId;
				// Is this our game?
				if (GameId == LanGameUniqueId)
				{
					uint8 SQ1 = 0;
					PacketReader >> SQ1;
					uint8 SQ2 = 0;
					PacketReader >> SQ2;
					// Is this a server response?
					if (SQ1 == LAN_SERVER_RESPONSE1 && SQ2 == LAN_SERVER_RESPONSE2)
					{
						uint64 Nonce = 0;
						PacketReader >> Nonce;
						
						if (Nonce == LanNonce)
						{
							FGuid BeaconGuid;
							PacketReader >> BeaconGuid;

							bIsValid = !PacketReader.HasOverflow();

							if (bIsValid)
							{
								ResponseGuid = BeaconGuid;
							}
						}
					}
				}
			}
		}
	}
	return bIsValid;
}
