// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

/**
 * Online base module class
 */
class FOnlineBaseModule : public IModuleInterface
{
public:
	// IModuleInterface
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}
};

IMPLEMENT_MODULE(FOnlineBaseModule, OnlineBase);
