// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"

#define UE_API ONLINEBASE_API

/**
 * Generates a random nonce (number used once) of the desired length
 *
 * @param Nonce the buffer that will get the randomized data
 * @param Length the number of bytes to generate random values for
 */
inline void GenerateNonce(uint8* Nonce, uint32 Length)
{
	//@todo joeg -- switch to CryptGenRandom() if possible or something equivalent
		// Loop through generating a random value for each byte
	for (uint32 NonceIndex = 0; NonceIndex < Length; NonceIndex++)
	{
		Nonce[NonceIndex] = (uint8)(FMath::Rand() & 255);
	}
}

/**
 * This value indicates which packet version the server is sending. Clients with
 * differing versions will ignore these packets. This prevents crashing when
 * changing the packet format and there are existing servers on the network
 * Current format:
 *
 *	<Ver byte><Platform byte><Game unique 4 bytes><packet type 2 bytes><nonce 8 bytes><guid 16 bytes><payload>
 */
enum class ELANBeaconVersionHistory : uint8
{
	HISTORY_INITIAL				= 11,
	HISTORY_BEACON_GUID			= 12,	// Lan response packets now contain an instance guid
	
	// -----<new versions can be added before this line>-------------------------------------------------
	HISTORY_PLUS_ONE,
	HISTORY_LATEST 							= HISTORY_PLUS_ONE - 1
};

static const uint8 LAN_BEACON_PACKET_VERSION = (uint8)ELANBeaconVersionHistory::HISTORY_LATEST;

/** The size of the header for validation */
#define LAN_BEACON_PACKET_HEADER_SIZE 32
	
// Offsets for various fields
#define LAN_BEACON_VER_OFFSET 0
#define LAN_BEACON_PLATFORM_OFFSET 1
#define LAN_BEACON_GAMEID_OFFSET 2
#define LAN_BEACON_PACKETTYPE1_OFFSET 6
#define LAN_BEACON_PACKETTYPE2_OFFSET 7
#define LAN_BEACON_NONCE_OFFSET 8
#define LAN_BEACON_GUID_OFFSET 16

// Packet types in 2 byte readable form
#define LAN_SERVER_QUERY1 (uint8)'S'
#define LAN_SERVER_QUERY2 (uint8)'Q'

#define LAN_SERVER_RESPONSE1 (uint8)'S'
#define LAN_SERVER_RESPONSE2 (uint8)'R'

class FInternetAddr;
class FNboSerializeToBuffer;

// LAN Session Delegates
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnValidQueryPacket, uint8*, int32, uint64);
typedef FOnValidQueryPacket::FDelegate FOnValidQueryPacketDelegate;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnValidResponsePacket, uint8*, int32);
typedef FOnValidResponsePacket::FDelegate FOnValidResponsePacketDelegate;

DECLARE_MULTICAST_DELEGATE(FOnSearchingTimeout);
typedef FOnSearchingTimeout::FDelegate FOnSearchingTimeoutDelegate;

/** Enum indicating the state the LAN beacon is in */
namespace ELanBeaconState
{
	enum Type
	{
		/** The lan beacon is disabled */
		NotUsingLanBeacon,
		/** The lan beacon is responding to client requests for information */
		Hosting,
		/** The lan beacon is querying servers for information */
		Searching
	};

	/** @return the stringified version of the enum passed in */
	inline const TCHAR* ToString(ELanBeaconState::Type EnumVal)
	{
		switch (EnumVal)
		{

		case NotUsingLanBeacon:
		{
			return TEXT("NotUsingLanBeacon");
		}
		case Hosting:
		{
			return TEXT("Hosting");
		}
		case Searching:
		{
			return TEXT("Searching");
		}
		}
		return TEXT("");
	}
}

/**
 * Class responsible for sending/receiving UDP broadcasts for LAN match
 * discovery
 */
class FLanBeacon
{
	/** Builds the broadcast address and caches it */
	TSharedPtr<class FInternetAddr> BroadcastAddr;
	/** The socket to listen for requests on */
	class FSocket* ListenSocket;
	/** The address in bound requests come in on */
	TSharedPtr<class FInternetAddr> ListenAddr;
	/** Temporary address when receiving packets*/
	TSharedRef<class FInternetAddr> SockAddr;

public:
	/** Sets the broadcast address for this object */
	UE_API FLanBeacon();

	/** Frees the broadcast socket */
	UE_API ~FLanBeacon();

	/** Return true if there is a valid ListenSocket */
	UE_API bool IsListenSocketValid() const;

	/**
	 * Initializes the socket
	 *
	 * @param Port the port to listen on
	 *
	 * @return true if both socket was created successfully, false otherwise
	 */
	UE_API bool Init(int32 Port);

	/**
	 * Called to poll the socket for pending data. Any data received is placed
	 * in the specified packet buffer
	 *
	 * @param PacketData the buffer to get the socket's packet data
	 * @param BufferSize the size of the packet buffer
	 *
	 * @return the number of bytes read (0 if none or an error)
	 */
	UE_API int32 ReceivePacket(uint8* PacketData, int32 BufferSize);

	/**
	 * Uses the cached broadcast address to send packet to a subnet
	 *
	 * @param Packet the packet to send
	 * @param Length the size of the packet to send
	 */
	UE_API bool BroadcastPacket(uint8* Packet, int32 Length);
};

#define LAN_ANNOUNCE_PORT 14001
#define LAN_UNIQUE_ID 9999
#define LAN_QUERY_TIMEOUT 5
#define LAN_QUERY_RETRY_TIME 1
#define LAN_PLATFORMMASK 0xffffffff

/**
 *	Encapsulate functionality related to LAN broadcast data
 */
class FLANSession
{
protected:
	/**
	 * Determines if the packet header is valid or not
	 *
	 * @param Packet the packet data to check
	 * @param Length the size of the packet buffer
	 *
	 * @return true if the header is valid, false otherwise
	 */
	UE_API bool IsValidLanQueryPacket(const uint8* Packet, uint32 Length, uint64& ClientNonce);

	/**
	 * Determines if the packet header is valid or not
	 *
	 * @param Packet the packet data to check
	 * @param Length the size of the packet buffer
	 * @param ResponseGuid the beacon guid from the response packet
	 *
	 * @return true if the header is valid, false otherwise
	 */
	UE_API bool IsValidLanResponsePacket(const uint8* Packet, uint32 Length, FGuid& ResponseGuid);

public:

	/** Port to listen on for LAN queries/responses */
	int32 LanAnnouncePort;

	/** Unique id to keep UE games from seeing each others' LAN packets */
	int32 LanGameUniqueId;

	/** Mask containing which platforms can cross communicate */
	int32 LanPacketPlatformMask;

	/** The amount of time to wait before timing out a LAN query request */
	float LanQueryTimeout;

	/** The amount of time to wait before resending a LAN query request */
	float LanQueryRetryTime;

	/** LAN beacon for packet broadcast */
	class FLanBeacon* LanBeacon;

	/** State of the LAN beacon */
	ELanBeaconState::Type LanBeaconState;

	/** Used by a client to uniquely identify itself during LAN match discovery */
	uint64 LanNonce;

	/** The amount of time before the LAN query is considered done */
	float LanQueryTimeLeft;

	/** The amount of time before the LAN query retry */
	float LanQueryRetryTimeLeft;

	/** Cached search packet data for retries */
	TArray<uint8> RetryData;

	/** Unique identifier for this hosting/searching LAN session */
	FGuid LanBeaconGuid;

	FLANSession() :
		LanAnnouncePort(LAN_ANNOUNCE_PORT),
		LanGameUniqueId(LAN_UNIQUE_ID),
		LanPacketPlatformMask(LAN_PLATFORMMASK),
		LanQueryTimeout(LAN_QUERY_TIMEOUT),
		LanQueryRetryTime(LAN_QUERY_RETRY_TIME),
		LanBeacon(nullptr),
		LanBeaconState(ELanBeaconState::NotUsingLanBeacon),
		LanNonce(0),
		LanQueryTimeLeft(0.0f),
		LanQueryRetryTimeLeft(0.0f)
	{
		if (!GConfig->GetInt(TEXT("LANSession"), TEXT("LanAnnouncePort"), LanAnnouncePort, GEngineIni))
		{
			LanAnnouncePort = LAN_ANNOUNCE_PORT;
		}
		if (!GConfig->GetInt(TEXT("LANSession"), TEXT("LanGameUniqueId"), LanGameUniqueId, GEngineIni))
		{
			LanGameUniqueId = LAN_UNIQUE_ID;
		}
	}

	virtual ~FLANSession()
	{
		StopLANSession();
	}

	/**
	 * Creates the LAN beacon for queries/advertising servers
	 *
	 * @param QueryDelegate delegate to fire when a client query is received
	 */
	UE_API bool Host(FOnValidQueryPacketDelegate& QueryDelegate);

	/**
	 * Creates the LAN beacon for queries/advertising servers
	 *
	 * @param Packet packet to be search when broadcasting the search
	 * @param ResponseDelegate delegate to fire when a server response is received
	 * @param TimeoutDelegate delegate to fire if we exceed maximum search time
	 *
	 * @return true if search was started successfully, false otherwise
	 */
	UE_API bool Search(class FNboSerializeToBuffer& Packet, FOnValidResponsePacketDelegate& ResponseDelegate, FOnSearchingTimeoutDelegate& TimeoutDelegate);

	/**
	 * Stops the LAN beacon from accepting broadcasts
	 */
	UE_API void StopLANSession();

	UE_API void Tick(float DeltaTime);

	/** create packet of MAX size */
	UE_API void CreateHostResponsePacket(FNboSerializeToBuffer& Packet, uint64 ClientNonce);
	UE_API void CreateClientQueryPacket(FNboSerializeToBuffer& Packet, uint64 ClientNonce);

	/**
	 * Uses the cached broadcast address to send packet to a subnet
	 *
	 * @param Packet the packet to send
	 * @param Length the size of the packet to send
	 */
	UE_API bool BroadcastPacket(uint8* Packet, int32 Length);

	ELanBeaconState::Type GetBeaconState() const
	{
		return LanBeaconState;
	}

	FOnValidQueryPacket OnValidQueryPacketDelegates;
	FOnValidResponsePacket OnValidResponsePacketDelegates;
	FOnSearchingTimeout OnSearchTimeoutDelegates;
};

#undef UE_API
