// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "IPAddress.h"

#define UE_API ONLINEBASE_API

namespace NboSerializer
{
	template <typename T>
	static constexpr inline bool UnexpectedType = false;
}

/**
 * Serializes data in network byte order form into a buffer
 */
class FNboSerializeToBuffer
{
	/** Hidden on purpose */
	UE_API FNboSerializeToBuffer(void);

protected:
	/**
	 * Holds the data as it is serialized
	 */
	TArray<uint8> Data;
	/**
	 * Tracks how many bytes have been written in the packet
	 */
	uint32 NumBytes;

	/** Indicates whether writing to the buffer caused an overflow or not */
	bool bHasOverflowed;

public:
	/**
	 * Initializes the write tracking
	 */
	FNboSerializeToBuffer(uint32 Size) :
		NumBytes(0),
		bHasOverflowed(false)
	{
		Data.Empty(Size);
		Data.AddZeroed(Size);
	}

	/**
	 * Cast operator to get at the formatted buffer data
	 */
	inline operator uint8*(void) const
	{
		return (uint8*)Data.GetData();
	}

	/**
	 * Cast operator to get at the formatted buffer data
	 */
	inline const TArray<uint8>& GetBuffer(void) const
	{
		return Data;
	}

	/**
	 * Returns the size of the buffer we've built up
	 */
	inline uint32 GetByteCount(void) const
	{
		return NumBytes;
	}

	/**
	 * Returns the number of bytes preallocated in the array
	 */
	inline uint32 GetBufferSize(void) const
	{
		return (uint32)Data.Num();
	}

	/**
	 * Trim any extra space in the buffer that is not used
	 */
	inline void TrimBuffer()
	{
		if (GetBufferSize() > GetByteCount())
		{
			Data.RemoveAt(GetByteCount(), GetBufferSize()-GetByteCount());
		}
	}

	/**
	 * Act as if the buffer is now empty. Useful for buffers that are reused
	 */
	inline void Reset()
	{
		NumBytes = 0;
		bHasOverflowed = false;
	}

	/**
	 * Adds a bool to the buffer (converted to uint8)
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar, const bool B)
	{
		Ar << (uint8)B;

		return Ar;
	}

	/**
	 * Adds a char to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const char Ch)
	{
		if (!Ar.HasOverflow() && Ar.NumBytes + 1 <= Ar.GetBufferSize())
		{
			Ar.Data[Ar.NumBytes++] = Ch;
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Adds a byte to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const uint8& B)
	{
		if (!Ar.HasOverflow() && Ar.NumBytes + 1 <= Ar.GetBufferSize())
		{
			Ar.Data[Ar.NumBytes++] = B;
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Adds a byte to the buffer
	 */
	template<class TEnum>
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const TEnumAsByte<TEnum>& B)
	{
		return Ar << *(uint8*)&B;
	}

	/**
	 * Adds an int32 to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const int32& I)
	{
		return Ar << *(uint32*)&I;
	}

	/**
	 * Adds a uint32 to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const uint32& D)
	{
		if (!Ar.HasOverflow() && Ar.NumBytes + 4 <= Ar.GetBufferSize())
		{
			Ar.Data[Ar.NumBytes + 0] = (D >> 24) & 0xFF;
			Ar.Data[Ar.NumBytes + 1] = (D >> 16) & 0xFF;
			Ar.Data[Ar.NumBytes + 2] = (D >> 8) & 0xFF;
			Ar.Data[Ar.NumBytes + 3] = D & 0xFF;
			Ar.NumBytes += 4;
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Adds a uint64 to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const uint64& Q)
	{
		if (!Ar.HasOverflow() && Ar.NumBytes + 8 <= Ar.GetBufferSize())
		{
			Ar.Data[Ar.NumBytes + 0] = (Q >> 56) & 0xFF;
			Ar.Data[Ar.NumBytes + 1] = (Q >> 48) & 0xFF;
			Ar.Data[Ar.NumBytes + 2] = (Q >> 40) & 0xFF;
			Ar.Data[Ar.NumBytes + 3] = (Q >> 32) & 0xFF;
			Ar.Data[Ar.NumBytes + 4] = (Q >> 24) & 0xFF;
			Ar.Data[Ar.NumBytes + 5] = (Q >> 16) & 0xFF;
			Ar.Data[Ar.NumBytes + 6] = (Q >> 8) & 0xFF;
			Ar.Data[Ar.NumBytes + 7] = Q & 0xFF;
			Ar.NumBytes += 8;
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Adds a float to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const float& F)
	{
		uint32 Value = *(uint32*)&F;
		return Ar << Value;
	}

	/**
	 * Adds a double to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const double& Dbl)
	{
		uint64 Value = *(uint64*)&Dbl;
		return Ar << Value;
	}

	/**
	 * Adds a FString to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const FString& String)
	{
		// We send strings length prefixed
		FTCHARToUTF8 Converted(*String);
		int32 Len = Converted.Length();

		Ar << Len;

		if (!Ar.HasOverflow() && Ar.NumBytes + Len <= Ar.GetBufferSize())
		{
			// Handle empty strings
			if (Len > 0)
			{
				ANSICHAR* Ptr = (ANSICHAR*)Converted.Get();

				// memcpy it into the buffer
				FMemory::Memcpy(&Ar.Data[Ar.NumBytes], Ptr,Len);
				Ar.NumBytes += Len;
			}
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Adds a string to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const TCHAR* String)
	{
		// We send strings length prefixed (conversion handles null pointers)
		FTCHARToUTF8 Converted(String);
		int32 Len = Converted.Length();

		Ar << Len;

		if (!Ar.HasOverflow() && Ar.NumBytes + Len <= Ar.GetBufferSize())
		{
			// Handle empty/null strings
			if (Len > 0)
			{
				ANSICHAR* Ptr = (ANSICHAR*)Converted.Get();

				// memcpy it into the buffer
				FMemory::Memcpy(&Ar.Data[Ar.NumBytes], Ptr, Len);
				Ar.NumBytes += Len;
			}
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Adds a substring to the buffer
	 */
	inline FNboSerializeToBuffer& AddString(const ANSICHAR* String,const int32 Length)
	{
		// We send strings length prefixed
		int32 Len = Length;
		(*this) << Len;

		if (!HasOverflow() && NumBytes + Len <= GetBufferSize())
		{
			// Don't process if null
			if (String)
			{
				// memcpy it into the buffer
				FMemory::Memcpy(&Data[NumBytes],String,Len);
				NumBytes += Len;
			}
		}
		else
		{
			bHasOverflowed = true;
		}

		return *this;
	}

	/**
	 * Adds an FName to the buffer as a string
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const FName& Name)
	{
		const FString NameString = Name.ToString();
		Ar << NameString;
		return Ar;
	}

	/**
	 * Adds an ip address to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const FInternetAddr& Addr)
	{
		TArray<uint8> RawAddressBytes = Addr.GetRawIp();
		Ar << RawAddressBytes.Num();
		Ar.WriteBinary(RawAddressBytes.GetData(), RawAddressBytes.Num());
		Ar << Addr.GetPort();
		return Ar;
	}

	/**
	 * Adds a FGuid to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const FGuid& Guid)
	{
		Ar << Guid.A;
		Ar << Guid.B;
		Ar << Guid.C;
		Ar << Guid.D;
		return Ar;
	}

	// This will be chosen in case of any type not specially supported
	template <typename T>
	friend inline FNboSerializeToBuffer& operator <<(FNboSerializeToBuffer& Ar, const T& Value)
	{
		static_assert(NboSerializer::UnexpectedType<T>, "Unsupported type in FNboSerializeToBuffer::operator<<");

		return Ar;
	}

	/**
	 * Writes a blob of data to the buffer
	 *
	 * @param Buffer the source data to append
	 * @param NumToWrite the size of the blob to write
	 */
	inline void WriteBinary(const uint8* Buffer,uint32 NumToWrite)
	{
		if (!HasOverflow() && NumBytes + NumToWrite <= GetBufferSize())
		{
			FMemory::Memcpy(&Data[NumBytes],Buffer,NumToWrite);
			NumBytes += NumToWrite;
		}
		else
		{
			bHasOverflowed = true;
		}
	}

	/**
	 * Gets the buffer at a specific point
	 */
	inline uint8* GetRawBuffer(uint32 Offset)
	{
		return &Data[Offset];
	}

	/**
	 * Skips forward in the buffer by the specified amount
	 *
	 * @param Amount the number of bytes to skip ahead
	 */
	inline void SkipAheadBy(uint32 Amount)
	{
		if (!HasOverflow() && NumBytes + Amount <= GetBufferSize())
		{
			NumBytes += Amount;
		}
		else
		{
			bHasOverflowed = true;
		}
	}

	/** Returns whether the buffer had an overflow when writing to it */
	inline bool HasOverflow(void) const
	{
		return bHasOverflowed;
	}
};

MSVC_PRAGMA(warning(push))
// Disable used without initialization warning because the reads are initializing
MSVC_PRAGMA(warning(disable : 4700))

/**
 * Class used to read data from a NBO data buffer
 */
class FNboSerializeFromBuffer
{
protected:
	/** Pointer to the data this reader is attached to */
	const uint8* Data;
	/** The size of the data in bytes */
	const int32 NumBytes;
	/** The current location in the byte stream for reading */
	int32 CurrentOffset;
	/** Indicates whether reading from the buffer caused an overflow or not */
	bool bHasOverflowed;

	/** Hidden on purpose */
	UE_API FNboSerializeFromBuffer(void);

public:
	/**
	 * Initializes the buffer, size, and zeros the read offset
	 *
	 * @param InData the buffer to attach to
	 * @param Length the size of the buffer we are attaching to
	 */
	FNboSerializeFromBuffer(const uint8* InData,int32 Length) :
		Data(InData),
		NumBytes(Length),
		CurrentOffset(0),
		bHasOverflowed(false)
	{
	}

	/**
	 * Reads a bool from the buffer (as uint8)
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar, bool& B)
	{
		uint8 Read = 0;
		Ar >> Read;
		B = !!Read;

		return Ar;
	}

	/**
	 * Reads a char from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,char& Ch)
	{
		if (!Ar.HasOverflow() && Ar.CurrentOffset + 1 <= Ar.NumBytes)
		{
			Ch = Ar.Data[Ar.CurrentOffset++];
		}
		else
		{
			Ar.bHasOverflowed = true;
		}
		return Ar;
	}

	/**
	 * Reads a byte from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,uint8& B)
	{
		if (!Ar.HasOverflow() && Ar.CurrentOffset + 1 <= Ar.NumBytes)
		{
			B = Ar.Data[Ar.CurrentOffset++];
		}
		else
		{
			Ar.bHasOverflowed = true;
		}
		return Ar;
	}
	/**
	 * Reads a byte from the buffer
	 */
	template<class TEnum>
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,TEnumAsByte<TEnum>& B)
	{
		return Ar >> *(uint8*)&B;
	}

	/**
	 * Reads an int32 from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,int32& I)
	{
		return Ar >> *(uint32*)&I;
	}

	/**
	 * Reads a uint32 from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,uint32& D)
	{
		if (!Ar.HasOverflow() && Ar.CurrentOffset + 4 <= Ar.NumBytes)
		{
			uint32 D1 = Ar.Data[Ar.CurrentOffset + 0];
			uint32 D2 = Ar.Data[Ar.CurrentOffset + 1];
			uint32 D3 = Ar.Data[Ar.CurrentOffset + 2];
			uint32 D4 = Ar.Data[Ar.CurrentOffset + 3];
			D = D1 << 24 | D2 << 16 | D3 << 8 | D4;
			Ar.CurrentOffset += 4;
		}
		else
		{
			Ar.bHasOverflowed = true;
		}
		return Ar;
	}

	/**
	 * Adds a uint64 to the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,uint64& Q)
	{
		if (!Ar.HasOverflow() && Ar.CurrentOffset + 8 <= Ar.NumBytes)
		{
			Q = ((uint64)Ar.Data[Ar.CurrentOffset + 0] << 56) |
				((uint64)Ar.Data[Ar.CurrentOffset + 1] << 48) |
				((uint64)Ar.Data[Ar.CurrentOffset + 2] << 40) |
				((uint64)Ar.Data[Ar.CurrentOffset + 3] << 32) |
				((uint64)Ar.Data[Ar.CurrentOffset + 4] << 24) |
				((uint64)Ar.Data[Ar.CurrentOffset + 5] << 16) |
				((uint64)Ar.Data[Ar.CurrentOffset + 6] << 8) |
				(uint64)Ar.Data[Ar.CurrentOffset + 7];
			Ar.CurrentOffset += 8;
		}
		else
		{
			Ar.bHasOverflowed = true;
		}
		return Ar;
	}

	/**
	 * Reads a float from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,float& F)
	{
		return Ar >> *(uint32*)&F;
	}

	/**
	 * Reads a double from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,double& Dbl)
	{
		return Ar >> *(uint64*)&Dbl;
	}

	/**
	 * Reads a FString from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,FString& String)
	{
		// We send strings length prefixed
		int32 Len = 0;
		Ar >> Len;

		// Check this way to trust NumBytes and CurrentOffset to be more accurate than the packet Len value
		const bool bSizeOk = (Len >= 0) && (Len <= (Ar.NumBytes - Ar.CurrentOffset));
		if (!Ar.HasOverflow() && bSizeOk)
		{
			// Handle strings of zero length
			if (Len > 0)
			{
				char* Temp = (char*)FMemory_Alloca(Len + 1);
				// memcpy it in from the buffer
				FMemory::Memcpy(Temp, &Ar.Data[Ar.CurrentOffset], Len);
				Temp[Len] = '\0';

				FUTF8ToTCHAR Converted(Temp);
				TCHAR* Ptr = (TCHAR*)Converted.Get();
				String = Ptr;
				Ar.CurrentOffset += Len;
			}
			else
			{
				String.Empty();
			}
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Reads an FName from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,FName& Name)
	{
		FString NameString; 
		Ar >> NameString;
		Name = FName(*NameString);
		return Ar;
	}

	/**
	 * Reads an ip address from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,FInternetAddr& Addr)
	{
		TArray<uint8> RawAddressBytes;
		uint32 AddressSize = 0;
		Ar >> AddressSize;
		Ar.ReadBinaryArray(RawAddressBytes, AddressSize);
		Addr.SetRawIp(RawAddressBytes);
		int32 InPort = 0;
		Ar >> InPort;
		Addr.SetPort(InPort);
		return Ar;
	}

	/**
	 * Reads a FGuid from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,FGuid& Guid)
	{
		Ar >> Guid.A;
		Ar >> Guid.B;
		Ar >> Guid.C;
		Ar >> Guid.D;
		return Ar;
	}

	// This will be chosen in case of any type not specially supported
	template <typename T>
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar, T& Value)
	{
		static_assert(NboSerializer::UnexpectedType<T>, "Unsupported type in FNboSerializeFromBuffer::operator>>");

		return Ar;
	}

	/**
	 * Reads a blob of data from the buffer into an array
	 * Prevents the necessity for additional allocation.
	 *
	 * @param OutArray the destination array
	 * @param NumToRead the size of the blob to read
	 */
	void ReadBinaryArray(TArray<uint8>& OutArray, uint32 NumToRead)
	{
		OutArray.Reserve(NumToRead);
		if (!HasOverflow() && CurrentOffset + (int32)NumToRead <= NumBytes)
		{
			for (uint32 i = 0; i < NumToRead; ++i)
			{
				OutArray.Add(Data[CurrentOffset + i]);
			}
			CurrentOffset += NumToRead;
		}
		else
		{
			bHasOverflowed = true;
		}
	}

	/**
	 * Reads a blob of data from the buffer
	 *
	 * @param OutBuffer the destination buffer
	 * @param NumToRead the size of the blob to read
	 */
	void ReadBinary(uint8* OutBuffer,uint32 NumToRead)
	{
		if (!HasOverflow() && CurrentOffset + (int32)NumToRead <= NumBytes)
		{
			FMemory::Memcpy(OutBuffer,&Data[CurrentOffset],NumToRead);
			CurrentOffset += NumToRead;
		}
		else
		{
			bHasOverflowed = true;
		}
	}

	/**
	 * Seek to the desired position in the buffer
	 *
	 * @param Pos the offset from the start of the buffer
	 */
	void Seek(int32 Pos)
	{
		checkSlow(Pos >= 0);

		if (!HasOverflow() && Pos < NumBytes)
		{
			CurrentOffset = Pos;
		}
		else
		{
			bHasOverflowed = true;
		}
	}

	/** @return Current position of the buffer being to be read */
	inline int32 Tell(void) const
	{
		return CurrentOffset;
	}

	/** Returns whether the buffer had an overflow when reading from it */
	inline bool HasOverflow(void) const
	{
		return bHasOverflowed;
	}

	/** @return Number of bytes remaining to read from the current offset to the end of the buffer */
	inline int32 AvailableToRead(void) const
	{
		return FMath::Max<int32>(0,NumBytes - CurrentOffset);
	}

	/**
	 * Returns the number of total bytes in the buffer
	 */
	inline int32 GetBufferSize(void) const
	{
		return NumBytes;
	}
};

MSVC_PRAGMA(warning(pop))

#undef UE_API
