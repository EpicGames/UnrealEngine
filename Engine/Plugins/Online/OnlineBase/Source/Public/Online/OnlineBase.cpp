// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineBase.h"

static TAutoConsoleVariable<int32> CVarBuildIdOverride(
TEXT("buildidoverride"),
0,
TEXT("Sets build id used for matchmaking "));

TAutoConsoleVariable<int32>& GetBuildIdOverrideCVar()
{
	return CVarBuildIdOverride;
}
