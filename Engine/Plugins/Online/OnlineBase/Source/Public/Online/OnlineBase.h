// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/IConsoleManager.h"

#ifndef ONLINE_SUCCESS
#define ONLINE_SUCCESS 0
#endif

#ifndef ONLINE_FAIL
#define ONLINE_FAIL (uint32)-1
#endif

#ifndef ONLINE_IO_PENDING
#define ONLINE_IO_PENDING 997
#endif

ONLINEBASE_API TAutoConsoleVariable<int32>&  GetBuildIdOverrideCVar();