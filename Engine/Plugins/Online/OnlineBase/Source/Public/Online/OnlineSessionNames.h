// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/CoreMiscDefines.h"
#include "UObject/NameTypes.h"

/** default beacon port, if not specified by other means */
#define DEFAULT_BEACON_PORT 15000

/**
 * Session settings
 */
/** Setting describing the name of the current map (value is FString) */
#define SETTING_MAPNAME FName(TEXT("MAPNAME"))
/** Setting describing the number of bots in the session (value is int32) */
#define SETTING_NUMBOTS FName(TEXT("NUMBOTS"))
/** Setting describing the game mode of the session (value is FString) */
#define SETTING_GAMEMODE FName(TEXT("GAMEMODE"))
/** Setting describing the beacon host port (value is int32) */
#define SETTING_BEACONPORT FName(TEXT("BEACONPORT"))
/** Server responds to Qos beacon requests (value is int32) */
#define SETTING_QOS FName(TEXT("QOS"))
/** Setting describing the region of the world you are in (value is FString) */
#define SETTING_REGION FName(TEXT("REGION"))
/** Setting describing the a specific subregion preference within a region (value is FString) */
#define SETTING_SUBREGION FName(TEXT("SUBREGION"))
/** Setting describing the unique id of a datacenter (value is FString) */
#define SETTING_DCID FName(TEXT("DCID"))
/** Number of players needed to fill out this session (value is int32) */
#define SETTING_NEEDS FName(TEXT("NEEDS"))
/** Second key for "needs" because can't set same value with two criteria (value is int32) */
#define SETTING_NEEDSSORT FName(TEXT("NEEDSSORT"))
/** Setting describing the session key (value is FString) */
#define SETTING_SESSIONKEY FName(TEXT("SESSIONKEY"))
/** Setting describing the session key (value is FString) */
#define SETTING_ALLOWBROADCASTING FName(TEXT("ALLOWBROADCASTING"))
/** Setting defining broadcaster (value is bool) */
#define SETTING_BROADCASTER FName(TEXT("BROADCASTER"))
/** Setting defining if we are a party member spectator (value is bool) */
#define SETTING_PARTYMEMBERSPECTATOR FName(TEXT("PARTYMEMBERSPECTATOR"))
/** Setting describing the number of spectator slots in the session (value is int32) */
#define SETTING_MAXSPECTATORS FName(TEXT("MAXSPECTATORS"))

/** 8 user defined integer params to be used when filtering searches for sessions */
#define SETTING_CUSTOMSEARCHINT1 FName(TEXT("CUSTOMSEARCHINT1"))
#define SETTING_CUSTOMSEARCHINT2 FName(TEXT("CUSTOMSEARCHINT2"))
#define SETTING_CUSTOMSEARCHINT3 FName(TEXT("CUSTOMSEARCHINT3"))
#define SETTING_CUSTOMSEARCHINT4 FName(TEXT("CUSTOMSEARCHINT4"))
#define SETTING_CUSTOMSEARCHINT5 FName(TEXT("CUSTOMSEARCHINT5"))
#define SETTING_CUSTOMSEARCHINT6 FName(TEXT("CUSTOMSEARCHINT6"))
#define SETTING_CUSTOMSEARCHINT7 FName(TEXT("CUSTOMSEARCHINT7"))
#define SETTING_CUSTOMSEARCHINT8 FName(TEXT("CUSTOMSEARCHINT8"))

/** Setting to allow non-primary platform members in a session */
#define SETTING_ALLOWEXTERNALMEMBERSINSESSION FName("ALLOWEXTERNALMEMBERSINSESSION")

/**
 * Setting describing whether host migration is enabled for the session.  This typically means when the host leaves, 
 * the online service picks a new host instead of destroying the session.  Not supported by all online services.
 */
#define SETTING_HOST_MIGRATION FName(TEXT("HOSTMIGRATION"))

/** TODO ONLINE Settings to consider */
/** The server's nonce for this session */
/** Whether the game is an invitation or searched for game */
/** The ping of the server in milliseconds (-1 means the server was unreachable) */
/** Whether this server is a dedicated server or not */
/** Represents how good a match this is in a range from 0 to 1 */
/** Whether there is a skill update in progress or not (don't do multiple at once) */
/** Whether to shrink the session slots when a player leaves the match or not */

// These are Xbox One specific setting to be used in FOnlineSessionSettings as keys; technically this can be put in OnlineSubsystemSettings.h

/** Custom settings to be associated with Session (value is FString)*/
#define SETTING_CUSTOM FName(TEXT("CUSTOM"))
/** Session is marked as using party enabled session flag (value is bool)*/
#define SETTING_PARTY_ENABLED_SESSION FName(TEXT("PARTYENABLEDSESSION"))
/** Hopper name to search within (value is FString) */
#define SETTING_MATCHING_HOPPER FName(TEXT("MATCHHOPPER"))
/** Timeout for search (value is float seconds) */
#define SETTING_MATCHING_TIMEOUT FName(TEXT("MATCHTIMEOUT"))
/** Match attributes (value is FString) */
#define SETTING_MATCHING_ATTRIBUTES	FName(TEXT("MATCHATTRIBUTES"))
/** Match attributes (value is FString) */
#define SETTING_MATCH_MEMBERS_JSON FName(TEXT("MATCHMEMBERS"))
/** Session member constant custom json (value is FString) - This is to be used with FString::Printf, populated with user xuid */
#define SETTING_SESSION_MEMBER_CONSTANT_CUSTOM_JSON_XUID_PREFIX TEXT("SESSIONMEMBERCONSTANTCUSTOMJSON")
/** Set self as host (value is int32)*/
#define SETTING_MAX_RESULT FName(TEXT("MAXRESULT"))
/** Set self as host (value is int32)*/
#define SETTING_CONTRACT_VERSION_FILTER FName(TEXT("CONTRACTVERSIONFILTER"))
/** Set self as host (value is int32)*/
#define SETTING_FIND_PRIVATE_SESSIONS FName(TEXT("FINDPRIVATESESSIONS"))
/** Set self as host (value is int32)*/
#define SETTING_FIND_RESERVED_SESSIONS FName(TEXT("FINDRESERVEDSESSIONS"))
/** Set self as host (value is int32)*/
#define SETTING_FIND_INACTIVE_SESSIONS FName(TEXT("FINDINACTIVESESSIONS"))
/** Set self as host (value is int32)*/
#define SETTING_MULTIPLAYER_VISIBILITY FName(TEXT("MULTIPLAYERVISIBILITY"))
/** Session template name, must match a session template name from the service config. */
#define SETTING_SESSION_TEMPLATE_NAME FName(TEXT("SESSIONTEMPLATENAME"))
/** Session change number from shoulder tapped version */
#define SETTING_CHANGE_NUMBER FName(TEXT("CHANGENUMBER"))
/** Session can attempt arbiter migration */
#define SETTING_ALLOW_ARBITER_MIGRATION FName(TEXT("ALLOWARBITERMIGRATION"))
/** Session preservation type for matchmaking */
#define SETTING_MATCHING_PRESERVE_SESSION FName(TEXT("PRESERVESESSIONALWAYS"))
/** Matchmade game session URI for join in progress/invites */
#define  SETTING_GAME_SESSION_URI FName(TEXT("GAMESESSIONURI"))
/** Session member group identifier (value is FString) - This is to be used with FString::Printf, populated with user xuid - Field required with Team Based matchmaking*/
#define SETTING_GROUP_NAME_PREFIX TEXT("USERGROUPNAME")
/** Information to join a third-party auxiliary session (value is FString) */
#define SETTING_CUSTOM_JOIN_INFO FName(TEXT("CUSTOMJOININFO"))
/** What verbosity of session updates to subscribe to (value is FString, comma-separated if multiple values needed)*/
#define SETTING_SESSION_SUBSCRIPTION_TYPES FName(TEXT("SESSIONSUBSCRIPTIONTYPE"))
/** Whether the session should be considered as the player's activity session or otherwise (value is bool) */
#define SETTING_ACTIVITY_SESSION FName(TEXT("ACTIVITYSESSION"))

/**
 * Search settings
 */

/** Search only for dedicated servers (value is true/false) */
#define SEARCH_DEDICATED_ONLY FName(TEXT("DEDICATEDONLY"))
/** Search for empty servers only (value is true/false) */
#define SEARCH_EMPTY_SERVERS_ONLY FName(TEXT("EMPTYONLY"))
/** Search for non empty servers only (value is true/false) */
#define SEARCH_NONEMPTY_SERVERS_ONLY FName(TEXT("NONEMPTYONLY"))
/** Search for secure servers only (value is true/false) */
#define SEARCH_SECURE_SERVERS_ONLY FName(TEXT("SECUREONLY"))

/** Search for a match with min player availability (value is int) */
#define SEARCH_MINSLOTSAVAILABLE FName(TEXT("MINSLOTSAVAILABLE"))
/** Exclude all matches where any unique ids in a given array are present (value is string of the form "uniqueid1;uniqueid2;uniqueid3") */
#define SEARCH_EXCLUDE_UNIQUEIDS FName(TEXT("EXCLUDEUNIQUEIDS"))
/** User ID to search for session of */
#define SEARCH_USER FName(TEXT("SEARCHUSER"))
/** Keywords to match in session search */
#define SEARCH_KEYWORDS FName(TEXT("SEARCHKEYWORDS"))
/** The matchmaking queue name to matchmake in, e.g. "TeamDeathmatch" (value is string) */
#define SEARCH_MATCHMAKING_QUEUE FName(TEXT("MATCHMAKINGQUEUE"))
/** If set, use the named Xbox Live hopper to find a session via matchmaking (value is a string) */
#define SEARCH_XBOX_LIVE_HOPPER_NAME UE_DEPRECATED_MACRO(5.4, "SEARCH_XBOX_LIVE_HOPPER_NAME has been deprecated. Use SETTING_MATCHING_HOPPER instead.") SETTING_MATCHING_HOPPER
/** Which session template from the service configuration to use.*/
#define SEARCH_XBOX_LIVE_SESSION_TEMPLATE_NAME UE_DEPRECATED_MACRO(5.4, "SEARCH_XBOX_LIVE_SESSION_TEMPLATE_NAME has been deprecated. Use SETTING_SESSION_TEMPLATE_NAME instead.")  SETTING_SESSION_TEMPLATE_NAME
/** Selection method used to determine which match to join when multiple are returned (valid only on Switch) */
#define SEARCH_SWITCH_SELECTION_METHOD FName(TEXT("SWITCHSELECTIONMETHOD"))
/** Whether to search for lobbies instead of sessions */
#define SEARCH_LOBBIES FName(TEXT("LOBBYSEARCH"))

// User attributes for searching (FSessionMatchmakingUser::Attributes)
/** Team a user is searching for */
#define SEARCH_USER_ATTRIBUTE_TEAM TEXT("TEAM")
